{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Microsoft GDK Store Plug-in for Unreal Engine",
	"Description": "Allows packaging Windows games for submission to the Xbox PC App",
	"CreatedBy": "Epic Games, Inc.",
	"Category": "Platform",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"RequiresBuildPlatform": false,
	"CanContainContent": false,
	"Installed": false,
	"SupportedTargetPlatforms": [ "Win64" ],
	"Modules": [
		{
			"Name": "MSGameStore",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [ "Win64" ]
		}
	],
	"Plugins": [
		{
			"Name": "MSGamingSupport",
			"Enabled": true
		}
	]
}