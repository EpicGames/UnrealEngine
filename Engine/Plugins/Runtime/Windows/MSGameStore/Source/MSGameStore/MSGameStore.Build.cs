// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using EpicGames.Core;

public class MSGameStore : ModuleRules
{
	public MSGameStore(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.Add("Core");
	}
}
