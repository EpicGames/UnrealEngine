// Copyright Epic Games, Inc. All Rights Reserved.

#include "MSGameStoreModule.h"
#include "HAL/Platform.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

THIRD_PARTY_INCLUDES_START
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include <appmodel.h>
#include "Microsoft/HideMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_END


class FMSGameStoreModule : public IMSGameStoreModule
{
public:

	virtual void StartupModule() override
	{
		bIsPackaged = DetectPackagedProcess();
	}

	virtual bool IsPackaged() const override
	{
		return bIsPackaged;
	}

	virtual IPlatformChunkInstall* GetChunkInstaller() const
	{
		if (!IsPackaged())
		{
			return nullptr;
		}

		IPlatformChunkInstallModule* PlatformChunkInstallModule = FModuleManager::LoadModulePtr<IPlatformChunkInstallModule>(TEXT("GDKPackageChunkInstall"));
		if (PlatformChunkInstallModule == nullptr)
		{
			return nullptr;
		}

		return PlatformChunkInstallModule->GetPlatformChunkInstall();
	}

private:

	// determines if we are an installed msixvc package without using the GDK runtime
	bool DetectPackagedProcess()
	{
		// editor will never be packaged 
#if WITH_EDITOR
		return false;
#else

		// get the kernel
		HMODULE hModule = ::GetModuleHandleW(TEXT("kernel32.dll"));
		if (hModule == nullptr)
		{
			return false;
		}

		// look up the GetCurrentPackageFullName function
PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS	// unsafe conversion from 'type of expression' to 'type required'
		typedef LONG(WINAPI *GetCurrentPackageFullNameProc)(UINT32*, PWSTR);
		GetCurrentPackageFullNameProc fnGetCurrentPackageFullName = (GetCurrentPackageFullNameProc)::GetProcAddress(hModule, "GetCurrentPackageFullName");
PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

		if (fnGetCurrentPackageFullName == nullptr)
		{
			return false;
		}

		// request buffer size
		UINT32 BufferLength = 0;
		LONG Result = fnGetCurrentPackageFullName(&BufferLength, nullptr);
		if (Result != ERROR_INSUFFICIENT_BUFFER)
		{
			return false;
		}

		// prepare a buffer and get the package full name
		TUniquePtr<WCHAR[]> Buffer = MakeUnique<WCHAR[]>(BufferLength);
		Result = fnGetCurrentPackageFullName(&BufferLength, Buffer.Get());
		if (Result != ERROR_SUCCESS)
		{
			return false;
		}

		// we are an appx package - log the package full name
		FString PackageFullName(Buffer.Get());
		UE_LOGF(LogInit, Log, "AppX Package: %ls", *PackageFullName);

		return true;
#endif //WITH_EDITOR
	}

	bool bIsPackaged = false;
};

IMPLEMENT_MODULE(FMSGameStoreModule, MSGameStore);
