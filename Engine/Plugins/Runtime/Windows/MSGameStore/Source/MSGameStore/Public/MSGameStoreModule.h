// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "GenericPlatform/GenericPlatformChunkInstall.h"

class MSGAMESTORE_API IMSGameStoreModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to the MS Game Store module instance
	 * @return Returns IMSGameStoreModule singleton instance, loading the module on demand if needed
	 */
	static inline IMSGameStoreModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IMSGameStoreModule>("MSGameStore");
	}

	/**
	 * Returns whether we are installed as an app package
	 */
	virtual bool IsPackaged() const = 0;


	/**
	 * Helper function to get the GDK chunk installer, if it is available
	 */
	virtual IPlatformChunkInstall* GetChunkInstaller() const = 0;
};

